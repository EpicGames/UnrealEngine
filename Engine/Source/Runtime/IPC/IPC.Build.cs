// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IPC : ModuleRules
{
	public IPC(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
	}
}
