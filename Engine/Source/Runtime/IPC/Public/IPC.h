// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================
	IPC.h: Public interface for inteprocess communication module
==============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"

namespace IPC
{
	/**
	 * Example class for synchronized interprocess memory
	 */
	class FSynchronizedInterprocessMemory
	{
		/** Lock that guards access to the memory region */
		FPlatformProcess::FSemaphore * Mutex;

		/** Low-level memory region */
		FPlatformMemory::FSharedMemoryRegion * Memory;

		/**
		 * Constructor
		 */
		FSynchronizedInterprocessMemory(FPlatformProcess::FSemaphore * InMutex, FPlatformMemory::FSharedMemoryRegion * InMemory)
			:	Mutex(InMutex)
			,	Memory(InMemory)
		{
			check(Mutex);
			check(Memory);
		}

	public:

		/**
		 * Destructor
		 */
		~FSynchronizedInterprocessMemory()
		{
			if (Mutex)
			{
				FPlatformProcess::DeleteInterprocessSynchObject(Mutex);
				Mutex = NULL;
			}

			if (Memory)
			{
				FPlatformMemory::UnmapNamedSharedMemoryRegion(Memory);
				Memory = NULL;
			}
		}

		/**
		 * Creates a new synchronized interprocess memory object
		 */
		static FSynchronizedInterprocessMemory * Create(const TCHAR* Name, SIZE_T Size)
		{
			FPlatformMemory::FSharedMemoryRegion * Memory = FPlatformMemory::MapNamedSharedMemoryRegion(Name, true,
				FPlatformMemory::ESharedMemoryAccess::Read | FPlatformMemory::ESharedMemoryAccess::Write, Size);

			if (!Memory)
			{
				return NULL;
			}

			// initialize memory with zeroes
			check(Memory->GetAddress());
			FMemory::Memzero(Memory->GetAddress(), Memory->GetSize());

			FPlatformProcess::FSemaphore * Mutex = FPlatformProcess::NewInterprocessSynchObject(Name, true);
			if (!Mutex)
			{
				FPlatformMemory::UnmapNamedSharedMemoryRegion(Memory);
				return NULL;
			}

			return new FSynchronizedInterprocessMemory(Mutex, Memory);
		}

		/**
		 * Opens existing synchronized interprocess memory object
		 */
		static FSynchronizedInterprocessMemory * OpenExisting(const TCHAR* Name, SIZE_T Size)
		{
			FPlatformMemory::FSharedMemoryRegion * Memory = FPlatformMemory::MapNamedSharedMemoryRegion(Name, false,
				FPlatformMemory::ESharedMemoryAccess::Read | FPlatformMemory::ESharedMemoryAccess::Write, Size);

			if (!Memory)
			{
				return NULL;
			}

			FPlatformProcess::FSemaphore * Mutex = FPlatformProcess::NewInterprocessSynchObject(Name, false);
			if (!Mutex)
			{
				FPlatformMemory::UnmapNamedSharedMemoryRegion(Memory);
				return NULL;
			}

			return new FSynchronizedInterprocessMemory(Mutex, Memory);
		}

		/** 
		 * Writes string to shared memory
		 * 
		 * @param String string to write
		 * @param MaxMillisecondsToWait max milliseconds to wait (if 0, wait forever)
		 *
		 * @return true if was able to write
		 */
		bool Write(const FString& String, uint32 MaxMillisecondsToWait = 0)
		{
			check(Mutex);
			check(Memory);

			// acquire
			if (!MaxMillisecondsToWait)
			{
				Mutex->Lock();
			}
			else
			{
				if (!Mutex->TryLock(MaxMillisecondsToWait * 1000000ULL))	// 1ms = 10^6 ns
				{
					return false;
				}
			}

			// we have exclusive ownership now!
			TCHAR* RawMemory = reinterpret_cast< TCHAR* >(Memory->GetAddress() );
			FCString::Strncpy(RawMemory, *String, Memory->GetSize());

			// relinquish
			Mutex->Unlock();

			return true;
		}

		/**
		 * Reads a string from shared memory
		 *
		 * @param String string to read into
		 * @param MaxMillisecondsToWait max milliseconds to wait (if 0, wait forever)
		 *
		 * @return true if read withing given time
		 */
		bool Read(FString& OutString, uint32 MaxMillisecondsToWait = 0)
		{
			check(Mutex);
			check(Memory);

			// acquire
			if (!MaxMillisecondsToWait)
			{
				Mutex->Lock();
			}
			else
			{
				if (!Mutex->TryLock(MaxMillisecondsToWait * 1000000ULL))	// 1ms = 10^6 ns
				{
					return false;
				}
			}

			// we have exclusive ownership now!
			const TCHAR* RawMemory = reinterpret_cast< const TCHAR* >( Memory->GetAddress() );
			OutString = FString(RawMemory);

			// relinquish
			Mutex->Unlock();

			return true;
		}
	};
}; // namespace IPC
